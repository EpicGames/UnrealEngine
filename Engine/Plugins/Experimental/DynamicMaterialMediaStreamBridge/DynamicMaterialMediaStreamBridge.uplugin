{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Material Designer Media Stream Bridge",
	"Description": "Integrates the Media Stream plugin with the Material Designer.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DynamicMaterialMediaStreamBridge",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DynamicMaterialMediaStreamBridgeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DynamicMaterial",
			"Enabled": true
		},
		{
			"Name": "MediaStream",
			"Enabled": true
		}
	]
}
