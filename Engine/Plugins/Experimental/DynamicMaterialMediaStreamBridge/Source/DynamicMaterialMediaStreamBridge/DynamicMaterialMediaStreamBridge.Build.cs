// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMaterialMediaStreamBridge : ModuleRules
{
	public DynamicMaterialMediaStreamBridge(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"DynamicMaterial"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"MediaStream"
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"SlateCore"
			});
		}

		ShortName = "DMMSBridge";
	}
}
