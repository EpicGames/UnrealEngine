// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMMaterialValueMediaStream.h"

#include "DMMaterialValueMediaStreamDynamic.h"
#include "Engine/Texture2D.h"
#include "IMediaStreamPlayer.h"
#include "MediaStream.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "Styling/SlateIconFinder.h"
#endif

#define LOCTEXT_NAMESPACE "DMMaterialValueMediaStream"

namespace UE::DynamicMaterialMediaStreamBridge::Private
{
	constexpr const TCHAR* DefaultTexturePath = TEXT("/Script/Engine.Texture2D'/Engine/EditorResources/SceneManager.SceneManager'");

	UTexture2D* GetDefaultTexture()
	{
		return LoadObject<UTexture2D>(GetTransientPackage(), DefaultTexturePath);
	}
}

UDMMaterialValueMediaStream::UDMMaterialValueMediaStream()
{
	MediaStream = CreateDefaultSubobject<UMediaStream>("MediaStream");

#if WITH_EDITOR
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueMediaStream, MediaStream));
#endif
}

UDMMaterialValueMediaStream::~UDMMaterialValueMediaStream()
{
}

UMediaStream* UDMMaterialValueMediaStream::GetMediaStream() const
{
	return MediaStream;
}

#if WITH_EDITOR
TSharedPtr<FJsonValue> UDMMaterialValueMediaStream::JsonSerialize() const
{
	if (!MediaStream)
	{
		return nullptr;
	}

	TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

	const FMediaStreamSource& Source = MediaStream->GetSource();
	
	Data.Add(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Scheme)), FDMJsonUtils::Serialize(Source.Scheme));
	Data.Add(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Path)), FDMJsonUtils::Serialize(Source.Path));

	if (TScriptInterface<IMediaStreamPlayer> Player = MediaStream->GetPlayer())
	{
		const FMediaStreamPlayerConfig& PlayerConfig = Player->GetPlayerConfig();
		Data.Add(FJsonObject::FStringType(TEXT("PlayerConfig")), FDMJsonUtils::Serialize<FMediaStreamPlayerConfig>(PlayerConfig));

		const FMediaStreamTextureConfig& TextureConfig = Player->GetTextureConfig();
		Data.Add(FJsonObject::FStringType(TEXT("TextureConfig")), FDMJsonUtils::Serialize<FMediaStreamTextureConfig>(TextureConfig));
	}

	return FDMJsonUtils::Serialize(Data);
}

bool UDMMaterialValueMediaStream::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	if (!MediaStream)
	{
		return false;
	}

	TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

	if (!FDMJsonUtils::Deserialize(InJsonValue, Data))
	{
		return false;
	}

	bool bSuccess = true;

	FMediaStreamSource Source;

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Scheme))))
	{
		FDMJsonUtils::Deserialize(*JsonValue, Source.Scheme);
	}
	else
	{
		bSuccess = false;
	}

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Path))))
	{
		FDMJsonUtils::Deserialize(*JsonValue, Source.Path);
	}
	else
	{
		bSuccess = false;
	}

	if (bSuccess)
	{
		MediaStream->SetSource(Source);
	}

	if (TScriptInterface<IMediaStreamPlayer> Player = MediaStream->GetPlayer())
	{
		if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(TEXT("PlayerConfig"))))
		{
			FMediaStreamPlayerConfig PlayerConfig;
			FDMJsonUtils::Deserialize<FMediaStreamPlayerConfig>(*JsonValue, PlayerConfig);
			Player->SetPlayerConfig(PlayerConfig);
		}

		if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(TEXT("TextureConfig"))))
		{
			FMediaStreamTextureConfig TextureConfig;
			FDMJsonUtils::Deserialize<FMediaStreamTextureConfig>(*JsonValue, TextureConfig);
			Player->SetTextureConfig(TextureConfig);
		}
	}

	return bSuccess;
}

void UDMMaterialValueMediaStream::ResetDefaultValue()
{
	DefaultValue = UE::DynamicMaterialMediaStreamBridge::Private::GetDefaultTexture();
}

UDMMaterialValueDynamic* UDMMaterialValueMediaStream::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueMediaStreamDynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueMediaStreamDynamic>(InMaterialModelDynamic, this);

	// Media stream value must be set to public so it can be referenced by the dynamic value.
	SetFlags(RF_Public);

	if (MediaStream)
	{
		// Media stream must be set to public so it can be referenced by the dynamic value.
		MediaStream->SetFlags(RF_Public);

		if (UMediaStream* MediaStreamDynamic = ValueDynamic->GetMediaStream())
		{
			const FMediaStreamSource& Source = MediaStream->GetSource();

			// Source needs to be public so it can be referenced by the dynamic value.
			if (Source.Object)
			{
				Source.Object->SetFlags(RF_Public);
			}

			MediaStreamDynamic->SetSource(Source);
		}
	}

	return ValueDynamic;
}
#endif

#if WITH_EDITOR
FString UDMMaterialValueMediaStream::GetComponentPathComponent() const
{
	return TEXT("MediaStream");
}

FText UDMMaterialValueMediaStream::GetComponentDescription() const
{
	return LOCTEXT("MediaStream", "Video");
}

void UDMMaterialValueMediaStream::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (IsValid(MediaStream))
	{
		MediaStream->EnsurePlayer();
	}

	UpdatePlayer();

	if (MediaStream)
	{
		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			MediaStreamPlayer->ApplyTextureConfig();
			MediaStreamPlayer->ApplyPlayerConfig();
		}
	}
}

FSlateIcon UDMMaterialValueMediaStream::GetComponentIcon() const
{
	return FSlateIconFinder::FindIcon("ClassIcon.MediaPlayer");
}

void UDMMaterialValueMediaStream::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == UMediaStream::GetSourcePropertyName())
	{
		Update(this, EDMUpdateType::Structure | EDMUpdateType::RefreshDetailView);
	}

	UpdatePlayer();
}

void UDMMaterialValueMediaStream::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(InPropertyChangedEvent);
}

void UDMMaterialValueMediaStream::PostLoad()
{
	Super::PostLoad();

	UpdatePlayer();
}

void UDMMaterialValueMediaStream::UpdateTextureFromMediaStream()
{
	UTexture* MediaTexture = nullptr;

	if (IsValid(MediaStream))
	{
		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			MediaTexture = MediaStreamPlayer->GetMediaTexture();
		}
	}

	if ((!MediaTexture && IsDefaultValue()) || (MediaTexture == GetValue()))
	{
		return;
	}

	Update(this, EDMUpdateType::RefreshDetailView);
	SetValue(MediaTexture);
}

void UDMMaterialValueMediaStream::SubscribeToEvents()
{
	if (IsValid(MediaStream))
	{
		MediaStream->GetOnSourceChanged().AddDynamic(this, &UDMMaterialValueMediaStream::OnSourceChanged);
		MediaStream->GetOnPlayerChanged().AddDynamic(this, &UDMMaterialValueMediaStream::OnPlayerChanged);
		SubscribedStreamWeak = MediaStream;
	}
	else
	{
		SubscribedStreamWeak.Reset();
	}
}

void UDMMaterialValueMediaStream::UnsubscribeFromEvents()
{
	if (UMediaStream* SubscribedStream = SubscribedStreamWeak.Get())
	{
		SubscribedStream->GetOnSourceChanged().RemoveAll(this);
		SubscribedStream->GetOnPlayerChanged().RemoveAll(this);
	}

	if (IsValid(MediaStream))
	{
		MediaStream->GetOnSourceChanged().RemoveAll(this);
		MediaStream->GetOnPlayerChanged().RemoveAll(this);
	}
}

void UDMMaterialValueMediaStream::OnSourceChanged(UMediaStream* InMediaStream)
{
	UpdatePlayer();
}

void UDMMaterialValueMediaStream::OnPlayerChanged(UMediaStream* InMediaStream)
{
	UpdatePlayer();
	Update(this, EDMUpdateType::Value);
}

void UDMMaterialValueMediaStream::UpdatePlayer()
{
	UnsubscribeFromEvents();
	SubscribeToEvents();
	UpdateTextureFromMediaStream();
}

void UDMMaterialValueMediaStream::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueMediaStream* Other = Cast<UDMMaterialValueMediaStream>(InOther);

	UMediaStream* OtherMediaStream = Other->GetMediaStream();

	if (!MediaStream || !OtherMediaStream)
	{
		return;
	}

	IMediaStreamPlayer* OtherPlayer = OtherMediaStream->GetPlayer().GetInterface();

	if (OtherPlayer && !OtherPlayer->IsReadOnly())
	{
		if (IMediaStreamPlayer* Player = MediaStream->GetPlayer().GetInterface())
		{
			if (OtherPlayer->GetTextureConfig() != Player->GetTextureConfig())
			{
				OtherPlayer->SetTextureConfig(Player->GetTextureConfig());
			}

			if (OtherPlayer->GetPlayerConfig() != Player->GetPlayerConfig())
			{
				OtherPlayer->SetPlayerConfig(Player->GetPlayerConfig());
			}
		}
	}

	if (OtherMediaStream->GetSource() != MediaStream->GetSource())
	{
		OtherMediaStream->SetSource(MediaStream->GetSource());
	}

	UpdatePlayer();
}

void UDMMaterialValueMediaStream::OnComponentAdded()
{
	Super::OnComponentAdded();

	UpdatePlayer();
}

void UDMMaterialValueMediaStream::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	UnsubscribeFromEvents();

	SetValue(nullptr);
}
#endif

#undef LOCTEXT_NAMESPACE
