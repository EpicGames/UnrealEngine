// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMMaterialValueMediaStreamDynamic.h"

#include "DMMaterialValueMediaStream.h"
#include "IMediaStreamPlayer.h"
#include "MediaStream.h"

#if WITH_EDITOR
#include "Styling/SlateIconFinder.h"
#endif

#define LOCTEXT_NAMESPACE "DMMaterialValueMediaStreamDynamic"

UDMMaterialValueMediaStreamDynamic::UDMMaterialValueMediaStreamDynamic()
{
	MediaStream = CreateDefaultSubobject<UMediaStream>("MediaStream");

#if WITH_EDITOR
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueMediaStreamDynamic, MediaStream));
#endif
}

#if WITH_EDITOR
TSharedPtr<FJsonValue> UDMMaterialValueMediaStreamDynamic::JsonSerialize() const
{
	if (!MediaStream)
	{
		return nullptr;
	}

	TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

	const FMediaStreamSource& Source = MediaStream->GetSource();

	Data.Add(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Scheme)), FDMJsonUtils::Serialize(Source.Scheme));
	Data.Add(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Path)), FDMJsonUtils::Serialize(Source.Path));

	if (TScriptInterface<IMediaStreamPlayer> Player = MediaStream->GetPlayer())
	{
		const FMediaStreamPlayerConfig& PlayerConfig = Player->GetPlayerConfig();
		Data.Add(FJsonObject::FStringType(TEXT("PlayerConfig")), FDMJsonUtils::Serialize<FMediaStreamPlayerConfig>(PlayerConfig));

		const FMediaStreamTextureConfig& TextureConfig = Player->GetTextureConfig();
		Data.Add(FJsonObject::FStringType(TEXT("TextureConfig")), FDMJsonUtils::Serialize<FMediaStreamTextureConfig>(TextureConfig));
	}

	return FDMJsonUtils::Serialize(Data);
}

bool UDMMaterialValueMediaStreamDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	if (!MediaStream)
	{
		return false;
	}

	TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

	if (!FDMJsonUtils::Deserialize(InJsonValue, Data))
	{
		return false;
	}

	bool bSuccess = true;

	FMediaStreamSource Source;

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Scheme))))
	{
		FDMJsonUtils::Deserialize(*JsonValue, Source.Scheme);
	}
	else
	{
		bSuccess = false;
	}

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(FMediaStreamSource, Path))))
	{
		FDMJsonUtils::Deserialize(*JsonValue, Source.Path);
	}
	else
	{
		bSuccess = false;
	}

	if (bSuccess)
	{
		MediaStream->SetSource(Source);
	}

	if (TScriptInterface<IMediaStreamPlayer> Player = MediaStream->GetPlayer())
	{
		if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(TEXT("PlayerConfig"))))
		{
			FMediaStreamPlayerConfig PlayerConfig;
			FDMJsonUtils::Deserialize<FMediaStreamPlayerConfig>(*JsonValue, PlayerConfig);
			Player->SetPlayerConfig(PlayerConfig);
		}

		if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(TEXT("TextureConfig"))))
		{
			FMediaStreamTextureConfig TextureConfig;
			FDMJsonUtils::Deserialize<FMediaStreamTextureConfig>(*JsonValue, TextureConfig);
			Player->SetTextureConfig(TextureConfig);
		}
	}

	return bSuccess;
}

FString UDMMaterialValueMediaStreamDynamic::GetComponentPathComponent() const
{
	return TEXT("MediaStream");
}

FText UDMMaterialValueMediaStreamDynamic::GetComponentDescription() const
{
	return LOCTEXT("MediaStream", "Video");
}

void UDMMaterialValueMediaStreamDynamic::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	UpdatePlayer();
}

FSlateIcon UDMMaterialValueMediaStreamDynamic::GetComponentIcon() const
{
	return FSlateIconFinder::FindIcon("ClassIcon.MediaPlayer");
}

void UDMMaterialValueMediaStreamDynamic::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	UpdatePlayer();
}

void UDMMaterialValueMediaStreamDynamic::PostLoad()
{
	Super::PostLoad();

	UpdatePlayer();
}

void UDMMaterialValueMediaStreamDynamic::UpdateTextureFromMediaStream()
{
	UTexture* MediaTexture = nullptr;

	if (IsValid(MediaStream))
	{
		if (IMediaStreamPlayer* MediaStreamPlayer = MediaStream->GetPlayer().GetInterface())
		{
			MediaTexture = MediaStreamPlayer->GetMediaTexture();
		}
	}

	SetValue(MediaTexture);
}

void UDMMaterialValueMediaStreamDynamic::SubscribeToEvents()
{
	if (IsValid(MediaStream))
	{
		MediaStream->GetOnSourceChanged().AddDynamic(this, &UDMMaterialValueMediaStreamDynamic::OnSourceChanged);
		MediaStream->GetOnPlayerChanged().AddDynamic(this, &UDMMaterialValueMediaStreamDynamic::OnPlayerChanged);
		SubscribedStreamWeak = MediaStream;
	}
	else
	{
		SubscribedStreamWeak.Reset();
	}
}

void UDMMaterialValueMediaStreamDynamic::UnsubscribeFromEvents()
{
	if (UMediaStream* SubscribedStream = SubscribedStreamWeak.Get())
	{
		SubscribedStream->GetOnSourceChanged().RemoveAll(this);
		SubscribedStream->GetOnPlayerChanged().RemoveAll(this);
	}

	if (IsValid(MediaStream))
	{
		MediaStream->GetOnSourceChanged().RemoveAll(this);
		MediaStream->GetOnPlayerChanged().RemoveAll(this);
	}
}

void UDMMaterialValueMediaStreamDynamic::OnSourceChanged(UMediaStream* InMediaStream)
{
	UpdatePlayer();

	Update(this, EDMUpdateType::RefreshDetailView);
}

void UDMMaterialValueMediaStreamDynamic::OnPlayerChanged(UMediaStream* InMediaStream)
{
	UpdatePlayer();

	Update(this, EDMUpdateType::Value);
}

void UDMMaterialValueMediaStreamDynamic::UpdatePlayer()
{
	UnsubscribeFromEvents();
	SubscribeToEvents();
	UpdateTextureFromMediaStream();
}

void UDMMaterialValueMediaStreamDynamic::OnComponentAdded()
{
	Super::OnComponentAdded();

	UpdatePlayer();
}

void UDMMaterialValueMediaStreamDynamic::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	UnsubscribeFromEvents();

	SetValue(nullptr);
}
#endif

void UDMMaterialValueMediaStreamDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueMediaStream* Other = Cast<UDMMaterialValueMediaStream>(InOther);

	UMediaStream* OtherMediaStream = Other->GetMediaStream();

	if (!MediaStream || !OtherMediaStream)
	{
		return;
	}

	IMediaStreamPlayer* OtherPlayer = OtherMediaStream->GetPlayer().GetInterface();

	if (OtherPlayer && !OtherPlayer->IsReadOnly())
	{
		if (IMediaStreamPlayer* Player = MediaStream->GetPlayer().GetInterface())
		{
			if (OtherPlayer->GetTextureConfig() != Player->GetTextureConfig())
			{
				OtherPlayer->SetTextureConfig(Player->GetTextureConfig());
			}

			if (OtherPlayer->GetPlayerConfig() != Player->GetPlayerConfig())
			{
				OtherPlayer->SetPlayerConfig(Player->GetPlayerConfig());
			}
		}
	}

	if (OtherMediaStream->GetSource() != MediaStream->GetSource())
	{
		OtherMediaStream->SetSource(MediaStream->GetSource());
	}
}

#undef LOCTEXT_NAMESPACE
