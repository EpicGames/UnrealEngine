// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialMediaStreamBridgeModule.h"

#include "Modules/ModuleManager.h"

void FDynamicMaterialMediaStreamBridgeModule::StartupModule()
{
}

void FDynamicMaterialMediaStreamBridgeModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FDynamicMaterialMediaStreamBridgeModule, DynamicMaterialMediaStreamBridge)
