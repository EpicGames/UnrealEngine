// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Media Stream Material Designer Bridge - Integrates the Media Stream plugin with the Material Designer.
 */
class FDynamicMaterialMediaStreamBridgeModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface
};
