// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "UObject/ScriptInterface.h"
#include "DMMaterialValueMediaStream.generated.h"

class IMediaStreamPlayer;
class UMediaStream;

/**
 * Component representing a render target texture value. Manages its own parameter.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueMediaStream : public UDMMaterialValueTexture
{
	GENERATED_BODY()

public:
	UDMMaterialValueMediaStream();

	virtual ~UDMMaterialValueMediaStream() override;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API  UMediaStream* GetMediaStream() const;

#if WITH_EDITOR
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API  virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API  virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable

	//~ Begin UDMMaterialValue
	virtual bool AllowEditValue() const { return false; }
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void ResetDefaultValue() override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual UDMMaterialValueDynamic* ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic) override;
	//~ End UDMMaterialValue

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual FSlateIcon GetComponentIcon() const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void PostLoad() override;
	//~ End UObject
#endif

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer|Media Stream", 
		meta = (AllowPrivateAccess = "true", NotKeyframeable))
	TObjectPtr<UMediaStream> MediaStream;

	TWeakObjectPtr<UMediaStream> SubscribedStreamWeak;

#if WITH_EDITOR
	void UpdateTextureFromMediaStream();

	void SubscribeToEvents();

	void UnsubscribeFromEvents();

	UFUNCTION()
	void OnSourceChanged(UMediaStream* InMediaStream);

	UFUNCTION()
	void OnPlayerChanged(UMediaStream* InMediaStream);

	void UpdatePlayer();

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API  virtual void OnComponentAdded() override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API  virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
#endif
};
