// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialValuesDynamic/DMMaterialValueTextureDynamic.h"

#include "DMMaterialValueMediaStreamDynamic.generated.h"

class IMediaStreamPlayer;
class UMediaStream;

/**
 * Link to a UDMMaterialValueMediaStream for Material Designer Model Dynamics.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueMediaStreamDynamic : public UDMMaterialValueTextureDynamic
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	using Super::PostEditorDuplicate;
#endif

	UDMMaterialValueMediaStreamDynamic();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UMediaStream* GetMediaStream() const { return MediaStream; }

#if WITH_EDITOR
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual FSlateIcon GetComponentIcon() const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API virtual void PostLoad() override;
	//~ End UObject
#endif

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer|Media Stream",
		meta = (AllowPrivateAccess = "true", NotKeyframeable))
	TObjectPtr<UMediaStream> MediaStream;

	TWeakObjectPtr<UMediaStream> SubscribedStreamWeak;

#if WITH_EDITOR
	void UpdateTextureFromMediaStream();

	void SubscribeToEvents();

	void UnsubscribeFromEvents();

	UFUNCTION()
	void OnSourceChanged(UMediaStream* InMediaStream);

	UFUNCTION()
	void OnPlayerChanged(UMediaStream* InMediaStream);

	void UpdatePlayer();

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API  virtual void OnComponentAdded() override;
	DYNAMICMATERIALMEDIASTREAMBRIDGE_API  virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
#endif	

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer
};
