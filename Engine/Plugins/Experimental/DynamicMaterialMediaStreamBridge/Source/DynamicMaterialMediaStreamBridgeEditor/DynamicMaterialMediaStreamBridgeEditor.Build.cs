// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMaterialMediaStreamBridgeEditor : ModuleRules
{
	public DynamicMaterialMediaStreamBridgeEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DynamicMaterial",
				"Engine"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DynamicMaterialEditor",
				"DynamicMaterialMediaStreamBridge",
				"MediaStream",
				"MediaStreamEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd"
			}
		);

		ShortName = "DMMSBridgeEditor";
	}
}
