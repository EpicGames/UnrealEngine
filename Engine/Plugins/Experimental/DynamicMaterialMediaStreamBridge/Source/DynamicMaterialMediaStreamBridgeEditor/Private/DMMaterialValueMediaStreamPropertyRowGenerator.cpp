// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMMaterialValueMediaStreamPropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "DMEDefs.h"
#include "DMMaterialValueMediaStream.h"
#include "IDynamicMaterialEditorModule.h"
#include "IMediaStreamPlayer.h"
#include "MediaStream.h"
#include "MediaStreamWidgets.h"
#include "Model/DynamicMaterialModelBase.h"
#include "UI/Utils/IDMWidgetLibrary.h"
#include "UObject/Object.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"

#define LOCTEXT_NAMESPACE "DMMaterialValueMediaStreamPropertyRowGenerator"

const TSharedRef<FDMMaterialValueMediaStreamPropertyRowGenerator>& FDMMaterialValueMediaStreamPropertyRowGenerator::Get()
{
	static TSharedRef<FDMMaterialValueMediaStreamPropertyRowGenerator> Generator = MakeShared<FDMMaterialValueMediaStreamPropertyRowGenerator>();
	return Generator;
}

void FDMMaterialValueMediaStreamPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialValueMediaStream* MediaStreamValue = Cast<UDMMaterialValueMediaStream>(InParams.Object);

	if (!MediaStreamValue)
	{
		return;
	}

	InParams.ProcessedObjects.Add(InParams.Object);

	UMediaStream* MediaStream = MediaStreamValue->GetMediaStream();

	if (!MediaStream)
	{
		return;
	}

	FDMComponentPropertyRowGeneratorParams MediaStreamParams = InParams;
	MediaStreamParams.Object = MediaStream;

	const bool bHasPlayer = !!MediaStream->GetPlayer().GetInterface();

	if (bHasPlayer)
	{
		AddControlCategory(MediaStreamParams);
	}

	AddSourceCategory(MediaStreamParams);

	if (bHasPlayer)
	{
		AddDetailsCategory(MediaStreamParams);
		AddCacheCategory(MediaStreamParams);
		AddPlayerProperty(MediaStreamParams);
	}
}

void FDMMaterialValueMediaStreamPropertyRowGenerator::AddControlCategory(FDMComponentPropertyRowGeneratorParams& InParams)
{
	FDMComponentPropertyRowGeneratorParams* Params = &InParams;
	UMediaStream* PreviewMediaStream = Cast<UMediaStream>(InParams.Object);

	if (!PreviewMediaStream)
	{
		return;
	}

	const FString RelativePath = InParams.Object->GetPathName(InParams.PreviewMaterialModelBase);
	UMediaStream* OriginalMediaStream = UDMMaterialModelFunctionLibrary::FindSubobject<UMediaStream>(InParams.OriginalMaterialModelBase, RelativePath);

	TArray<UMediaStream*> MediaStreams = !!OriginalMediaStream
		? TArray<UMediaStream*>({OriginalMediaStream, PreviewMediaStream})
		: TArray<UMediaStream*>({PreviewMediaStream});
	
	const FName CategoryName = "Media Controls";

	FDMPropertyHandle& TrackHandle = Params->PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(Params->CreatePropertyHandleParams(TEXT("Track")))
	);
	TrackHandle.CategoryOverrideName = CategoryName;
	TrackHandle.ValueWidget = UE::MediaStreamEditor::FMediaStreamWidgets::CreateTrackWidget(MediaStreams);
	TrackHandle.ValueName = "Track";

	FDMPropertyHandle& ControlHandle = Params->PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(Params->CreatePropertyHandleParams(TEXT("Controls")))
	);
	ControlHandle.CategoryOverrideName = CategoryName;
	ControlHandle.ValueWidget = UE::MediaStreamEditor::FMediaStreamWidgets::CreateControlsWidget(MediaStreams);
	ControlHandle.ValueName = "Controls";

	FDMPropertyHandle& PlayOnOpenHandle = Params->PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(Params->CreatePropertyHandleParams(TEXT("bPlayOnOpen")))
	);
	PlayOnOpenHandle.CategoryOverrideName = CategoryName;

	FDMPropertyHandle& LoopingHandle = Params->PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(Params->CreatePropertyHandleParams(TEXT("bLooping")))
	);
	LoopingHandle.CategoryOverrideName = CategoryName;
}

void FDMMaterialValueMediaStreamPropertyRowGenerator::AddSourceCategory(FDMComponentPropertyRowGeneratorParams& InParams)
{
	const FName CategoryName = "Media Source";

	UMediaStream* MediaStream = Cast<UMediaStream>(InParams.Object);

	if (!MediaStream)
	{
		return;
	}

	IMediaStreamSchemeHandler::FCustomWidgets Widgets = UE::MediaStreamEditor::FMediaStreamWidgets::GenerateSourceSchemeRows(MediaStream);

	for (const IMediaStreamSchemeHandler::FWidgetRow& WidgetRow : Widgets.CustomRows)
	{
		if (WidgetRow.Visibility.Get() != EVisibility::Visible)
		{
			continue;
		}

		FDMPropertyHandle Handle = IDynamicMaterialEditorModule::Get().GetWidgetLibrary().GetPropertyHandle(InParams.CreatePropertyHandleParams(WidgetRow.SourceProperty->GetFName()));
		Handle.bEnabled = WidgetRow.Enabled.Get();
		Handle.bKeyframeable = false;
		Handle.NameOverride = WidgetRow.SourceProperty->GetDisplayNameText();
		Handle.ValueName = *WidgetRow.Name.ToString();                           
		Handle.ValueWidget = WidgetRow.Widget;
		Handle.CategoryOverrideName = CategoryName;

		InParams.PropertyRows-> Add(Handle);
	}
}

void FDMMaterialValueMediaStreamPropertyRowGenerator::AddDetailsCategory(FDMComponentPropertyRowGeneratorParams& InParams)
{
	const FName CategoryName = "Media Details";

	UMediaStream* MediaStream = Cast<UMediaStream>(InParams.Object);

	FDMPropertyHandle& DetailsHandle = InParams.PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(TEXT("Details")))
	);
	DetailsHandle.CategoryOverrideName = CategoryName;
	DetailsHandle.ValueWidget = UE::MediaStreamEditor::FMediaStreamWidgets::CreateTextureDetailsWidget(MediaStream);
	DetailsHandle.ValueName = "Details";
}

void FDMMaterialValueMediaStreamPropertyRowGenerator::AddTextureCategory(FDMComponentPropertyRowGeneratorParams& InParams, 
	bool bInPreview)
{
	FDMComponentPropertyRowGeneratorParams* Params = &InParams;
	UMediaStream* MediaStream = nullptr;
	FDMComponentPropertyRowGeneratorParams OriginalParams = InParams;
	
	if (bInPreview)
	{
		MediaStream = Cast<UMediaStream>(InParams.Object);
	}
	else
	{
		const FString RelativePath = InParams.Object->GetPathName(InParams.PreviewMaterialModelBase);
		MediaStream = UDMMaterialModelFunctionLibrary::FindSubobject<UMediaStream>(InParams.OriginalMaterialModelBase, RelativePath);
		OriginalParams.Object = MediaStream;
		Params = &OriginalParams;
	}

	if (!MediaStream)
	{
		return;
	}
	
	const FName CategoryName = bInPreview
		? "Preview Media Texture"
		: "Source Media Texture";

	FDMPropertyHandle& MediaTextureHandle = Params->PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(Params->CreatePropertyHandleParams(TEXT("MediaTexture")))
	);
	MediaTextureHandle.CategoryOverrideName = CategoryName;
	MediaTextureHandle.ValueName = bInPreview ? "MediaTexturePreview" : "MediaTextureSource";

	if (bInPreview)
	{
		FDMPropertyHandle& TextureConfigHandle = Params->PropertyRows->Add_GetRef(
			IDMWidgetLibrary::Get().GetPropertyHandle(Params->CreatePropertyHandleParams(TEXT("TextureConfig")))
		);
		TextureConfigHandle.CategoryOverrideName = CategoryName;
	}
}

void FDMMaterialValueMediaStreamPropertyRowGenerator::AddCacheCategory(FDMComponentPropertyRowGeneratorParams& InParams)
{
	const FName CategoryName = "Media Cache";

	FDMPropertyHandle& UseLocalCacheHandle = InParams.PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(TEXT("bUseLocalCache")))
	);
	UseLocalCacheHandle.CategoryOverrideName = CategoryName;

	FDMPropertyHandle& CacheAheadHandle = InParams.PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(TEXT("CacheAhead")))
	);
	CacheAheadHandle.CategoryOverrideName = CategoryName;

	FDMPropertyHandle& CacheBehindHandle = InParams.PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(TEXT("CacheBehind")))
	);
	CacheBehindHandle.CategoryOverrideName = CategoryName;

	FDMPropertyHandle& CacheBehindGameHandle = InParams.PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(TEXT("CacheBehindGame")))
	);
	CacheBehindGameHandle.CategoryOverrideName = CategoryName;
}

void FDMMaterialValueMediaStreamPropertyRowGenerator::AddPlayerProperty(FDMComponentPropertyRowGeneratorParams& InParams)
{
	const FName CategoryName = "Media Player";

	FDMPropertyHandle& PlayerConfigHandle = InParams.PropertyRows->Add_GetRef(
		IDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(TEXT("PlayerConfig")))
	);
	PlayerConfigHandle.CategoryOverrideName = CategoryName;
}

#undef LOCTEXT_NAMESPACE
