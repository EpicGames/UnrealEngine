// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/PropertyGenerators/DMComponentPropertyRowGenerator.h"

class UMediaStream;

class FDMMaterialValueMediaStreamPropertyRowGenerator : public FDMComponentPropertyRowGenerator
{
public:
	static const TSharedRef<FDMMaterialValueMediaStreamPropertyRowGenerator>& Get();

	virtual ~FDMMaterialValueMediaStreamPropertyRowGenerator() override = default;

	//~ Begin FDMComponentPropertyRowGenerator
	virtual void AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams) override;
	//~ End FDMComponentPropertyRowGenerator

	void AddControlCategory(FDMComponentPropertyRowGeneratorParams& InParams);

	void AddSourceCategory(FDMComponentPropertyRowGeneratorParams& InParams);

	void AddDetailsCategory(FDMComponentPropertyRowGeneratorParams& InParams);

	void AddTextureCategory(FDMComponentPropertyRowGeneratorParams& InParams, bool bInPreview);

	void AddCacheCategory(FDMComponentPropertyRowGeneratorParams& InParams);

	void AddPlayerProperty(FDMComponentPropertyRowGeneratorParams& InParams);
};
