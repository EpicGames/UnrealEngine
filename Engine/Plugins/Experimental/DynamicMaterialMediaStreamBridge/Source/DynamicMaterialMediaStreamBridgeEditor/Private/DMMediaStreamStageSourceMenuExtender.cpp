// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMMediaStreamStageSourceMenuExtender.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "DMMaterialValueMediaStream.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UI/Menus/DMMenuContext.h"

#define LOCTEXT_NAMESPACE "DMMEdiaStreamStageSourceMenuExtender"

void FDMMediaStreamStageSourceMenuExtender::Integrate()
{
	if (bIntegrated)
	{
		return;
	}

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus)
	{
		return;
	}

	if (UToolMenu* AdvancedMenu = ToolMenus->ExtendMenu(TEXT("MaterialDesigner.MaterialStage.ChangeSource")))
	{
		FToolMenuSection& NewSection = AdvancedMenu->FindOrAddSection(TEXT("ChangeStageSource"), LOCTEXT("ChangeStageSource", "Change Stage Source"));

		NewSection.AddDynamicEntry(
			TEXT("MediaStream"),
			FNewToolMenuSectionDelegate::CreateRaw(this, &FDMMediaStreamStageSourceMenuExtender::ExtendMenu_ChangeSource)
		);
	}

	if (UToolMenu* AdvancedMenu = ToolMenus->ExtendMenu(TEXT("MaterialDesigner.MaterialStage")))
	{
		FToolMenuSection& NewSection = AdvancedMenu->FindOrAddSection(TEXT("ChangeStageSource"), LOCTEXT("ChangeStageSource", "Change Stage Source"));

		NewSection.AddDynamicEntry(
			TEXT("MediaStream"),
			FNewToolMenuSectionDelegate::CreateRaw(this, &FDMMediaStreamStageSourceMenuExtender::ExtendMenu_ChangeSource)
		);
	}

	if (UToolMenu* AdvancedMenu = ToolMenus->ExtendMenu(TEXT("MaterialDesigner.MaterialSlot.Layer")))
	{
		FToolMenuSection& NewSection = AdvancedMenu->FindOrAddSection(TEXT("AddLayer"), LOCTEXT("AddLayer", "Add Layer"));

		NewSection.AddDynamicEntry(
			TEXT("MediaStream"),
			FNewToolMenuSectionDelegate::CreateRaw(this, &FDMMediaStreamStageSourceMenuExtender::ExtendMenu_AddLayer)
		);
	}

	if (UToolMenu* AdvancedMenu = ToolMenus->ExtendMenu(TEXT("MaterialDesigner.MaterialSlot.AddLayer")))
	{
		FToolMenuSection& NewSection = AdvancedMenu->FindOrAddSection(TEXT("AddLayer"), LOCTEXT("AddLayer", "Add Layer"));

		NewSection.AddDynamicEntry(
			TEXT("MediaStream"),
			FNewToolMenuSectionDelegate::CreateRaw(this, &FDMMediaStreamStageSourceMenuExtender::ExtendMenu_AddLayer)
		);
	}

	bIntegrated = true;
}

void FDMMediaStreamStageSourceMenuExtender::ExtendMenu_ChangeSource(FToolMenuSection& InSection)
{
	UDMMenuContext* MenuContext = InSection.FindContext<UDMMenuContext>();

	if (!ensure(MenuContext))
	{
		return;
	}

	FToolMenuEntry& NewEntry = InSection.AddMenuEntry("MediaStream",
		LOCTEXT("ChangeSourceMediaStream", "Media"),
		LOCTEXT("ChangeSourceMediaStreamTooltip", "Change the source of this stage to a Media Stream."),
		GetDefault<UDMMaterialValueMediaStream>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this,
				&FDMMediaStreamStageSourceMenuExtender::ChangeSourceToMediaStreamFromContext,
				MenuContext
			)
		)
	);

	NewEntry.InsertPosition = FToolMenuInsert(TEXT("Noise"), EToolMenuInsertType::After);
}

void FDMMediaStreamStageSourceMenuExtender::ExtendMenu_AddLayer(FToolMenuSection& InSection)
{
	UDMMenuContext* MenuContext = InSection.FindContext<UDMMenuContext>();

	if (!ensure(MenuContext))
	{
		return;
	}

	FToolMenuEntry& NewEntry = InSection.AddMenuEntry("MediaStream",
		LOCTEXT("AddLayerMediaStream", "Media"),
		LOCTEXT("AddLayerMediaStreamTooltip", "Add a new layer based on a Media Stream."),
		GetDefault<UDMMaterialValueMediaStream>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this,
				&FDMMediaStreamStageSourceMenuExtender::AddMediaStreamLayerFromContext,
				MenuContext
			)
		)
	);

	NewEntry.InsertPosition = FToolMenuInsert(TEXT("Noise"), EToolMenuInsertType::Before);
}

void FDMMediaStreamStageSourceMenuExtender::ChangeSourceToMediaStreamFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = Stage->GetSource();

	if (!StageSource)
	{
		return;
	}

	UDMMaterialStageInputExpression* InputExpression = nullptr;

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		InputExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionTextureSample::StaticClass(),
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0,
			FDMMaterialStageConnectorChannel::THREE_CHANNELS
		);

		if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
		{
			if (UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask))
			{
				if (UDMMaterialStageThroughputLayerBlend* MaskLayerBlend = Cast<UDMMaterialStageThroughputLayerBlend>(MaskStage->GetSource()))
				{
					if (UDMMaterialStageInputExpression* MaskInputExpression = Cast<UDMMaterialStageInputExpression>(MaskLayerBlend->GetInputMask()))
					{
						UDMMaterialStageExpressionTextureSample* MaskTextureSample = Cast<UDMMaterialStageExpressionTextureSample>(MaskInputExpression->GetMaterialStageExpression());

						if (!MaskTextureSample)
						{
							MaskInputExpression->SetMaterialStageExpressionClass(UDMMaterialStageExpressionTextureSample::StaticClass());
							MaskTextureSample = Cast<UDMMaterialStageExpressionTextureSample>(MaskInputExpression->GetMaterialStageExpression());
						}

						if (MaskTextureSample)
						{
							MaskTextureSample->SetUseBaseTexture(true);
						}
					}
				}
			}
		}
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		InputExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionTextureSample::StaticClass(),
			UDMMaterialStageThroughputLayerBlend::InputMaskSource,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}

	if (!InputExpression)
	{
		return;
	}

	UDMMaterialSubStage* SubStage = InputExpression->GetSubStage();

	UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		SubStage,
		/* Texture input */ 0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		UDMMaterialValueMediaStream::StaticClass(),
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);
}

void FDMMediaStreamStageSourceMenuExtender::AddMediaStreamLayerFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialSlot* Slot = InMenuContext->GetSlot();

	if (!Slot)
	{
		return;
	}

	EDMMaterialPropertyType PropertyType = EDMMaterialPropertyType::None;

	if (UDMMaterialLayerObject* Layer = InMenuContext->GetLayer())
	{
		PropertyType = Layer->GetMaterialProperty();
	}
	else
	{
		const TArray<TObjectPtr<UDMMaterialLayerObject>>& Layers = Slot->GetLayers();

		if (Layers.IsEmpty())
		{
			return;
		}

		PropertyType = Layers.Last()->GetMaterialProperty();
	}

	if (GUndo)
	{
		Slot->Modify();
	}

	UDMMaterialLayerObject* NewLayer = nullptr;

	const FDMUpdateGuard Guard;

	NewLayer = Slot->AddDefaultLayer(PropertyType);

	if (!ensure(NewLayer))
	{
		return;
	}

	UDMMaterialStage* Stage = NewLayer->GetStage(EDMMaterialLayerStage::Base);

	if (!ensure(Stage))
	{
		return;
	}

	UDMMaterialStageInputExpression* NewExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		Stage,
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (!ensure(NewExpression))
	{
		return;
	}

	UDMMaterialSubStage* SubStage = NewExpression->GetSubStage();

	if (!ensure(SubStage))
	{
		return;
	}

	UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		SubStage,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		UDMMaterialValueMediaStream::StaticClass(),
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	NewLayer->Update(NewLayer, EDMUpdateType::Structure);

	if (UDMMaterialStage* MaskStage = NewLayer->GetStage(EDMMaterialLayerStage::Mask))
	{
		if (UDMMaterialStageThroughputLayerBlend* MaskLayerBlend = Cast<UDMMaterialStageThroughputLayerBlend>(MaskStage->GetSource()))
		{
			if (UDMMaterialStageInputExpression* MaskInputExpression = Cast<UDMMaterialStageInputExpression>(MaskLayerBlend->GetInputMask()))
			{
				UDMMaterialStageExpressionTextureSample* MaskTextureSample = Cast<UDMMaterialStageExpressionTextureSample>(MaskInputExpression->GetMaterialStageExpression());

				if (!MaskTextureSample)
				{
					MaskInputExpression->SetMaterialStageExpressionClass(UDMMaterialStageExpressionTextureSample::StaticClass());
					MaskTextureSample = Cast<UDMMaterialStageExpressionTextureSample>(MaskInputExpression->GetMaterialStageExpression());
				}

				if (MaskTextureSample)
				{
					MaskTextureSample->SetUseBaseTexture(true);
				}
			}
		}
	}
}

FDMMediaStreamStageSourceMenuExtender& FDMMediaStreamStageSourceMenuExtender::Get()
{
	static FDMMediaStreamStageSourceMenuExtender MenuExtension;
	return MenuExtension;
}

#undef LOCTEXT_NAMESPACE
