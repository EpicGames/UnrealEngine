// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UDMMenuContext;
struct FToolMenuSection;

class FDMMediaStreamStageSourceMenuExtender
{
public:
	static FDMMediaStreamStageSourceMenuExtender& Get();

	void Integrate();

private:
	bool bIntegrated = false;

	void ExtendMenu_ChangeSource(FToolMenuSection& InSection);

	void ExtendMenu_AddLayer(FToolMenuSection& InSection);

	void ChangeSourceToMediaStreamFromContext(UDMMenuContext* InMenuContext);

	void AddMediaStreamLayerFromContext(UDMMenuContext* InMenuContext);
};
