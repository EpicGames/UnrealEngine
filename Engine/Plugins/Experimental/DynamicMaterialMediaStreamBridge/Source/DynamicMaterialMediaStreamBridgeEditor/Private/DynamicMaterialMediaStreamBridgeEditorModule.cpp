// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialMediaStreamBridgeEditorModule.h"

#include "DMMaterialValueMediaStream.h"
#include "DMMaterialValueMediaStreamPropertyRowGenerator.h"
#include "DMMediaStreamStageSourceMenuExtender.h"
#include "IDynamicMaterialEditorModule.h"
#include "Modules/ModuleManager.h"

void FDynamicMaterialMediaStreamBridgeEditorModule::StartupModule()
{
	IDynamicMaterialEditorModule& DMEditorModule = IDynamicMaterialEditorModule::Get();
	DMEditorModule.RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialValueMediaStream, FDMMaterialValueMediaStreamPropertyRowGenerator>();

	FDMMediaStreamStageSourceMenuExtender::Get().Integrate();
}

IMPLEMENT_MODULE(FDynamicMaterialMediaStreamBridgeEditorModule, DynamicMaterialMediaStreamBridgeEditor)
